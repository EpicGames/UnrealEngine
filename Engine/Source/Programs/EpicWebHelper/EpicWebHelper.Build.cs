// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class EpicWebHelper : ModuleRules
{
	public EpicWebHelper(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"ApplicationCore",
				"Projects",
				"CEF3Utils",
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"CEF3"
			);
	}
}

