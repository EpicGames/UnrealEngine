// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;

[SupportedPlatforms("Win64", "Mac", "Linux")]
[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping)]
public class EpicWebHelperTarget : TargetRules
{
	public EpicWebHelperTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "EpicWebHelper";

		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		// Change the undecorated exe name to be the shipping one on windows
		UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			// CEF3 requires 4 copies of the helper app, each in theory has different system privileges granted by the codesigning tool
			// We don't support that yet, so just sym-link the 3 variants to our main helper binary here
			PostBuildSteps.Add(string.Format("echo Creating helper syminks"));
			var HelperTypes = new List<string>() { "GPU", "Renderer", "Plugin"};
			foreach( string HelperType in HelperTypes)
			{
				PostBuildSteps.Add(string.Format("cd $(EngineDir)/Binaries/$(TargetPlatform); if [ ! -h \"EpicWebHelper ({0})\" ]; then ln -s \"EpicWebHelper\" \"EpicWebHelper ({1})\"; fi", HelperType, HelperType));
			}
		}

		// Turn off various third party features we don't need

		// Currently we force Lean and Mean mode
		bBuildDeveloperTools = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bBuildWithEditorOnlyData = true;

		// Force all shader formats to be built and included.
		//bForceBuildShaderFormats = true;

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			// UE-201611 changing EpicWebHelper into a console application on Mac
			bIsBuildingConsoleApplication = true;
		}
		else
		{
			// CEFSubProcess is a Windows app (uses WinMain())
			bIsBuildingConsoleApplication = false;
		}

		// Disable logging, as the sub processes are spawned often and logging will just slow them down
		GlobalDefinitions.Add("ALLOW_LOG_FILE=0");

		// Enable OSX 10.11 support (CEF3 requires 10.11+ on macOS)
		bEnableOSX109Support = true;
		//ForceOSXMinVersion = "10.11";
	}
}
