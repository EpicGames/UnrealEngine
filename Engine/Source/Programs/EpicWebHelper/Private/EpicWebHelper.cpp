// Copyright Epic Games, Inc. All Rights Reserved.

//
// EpicWebHelper.cpp : Defines the entry point for the browser sub process
//

#include "EpicWebHelper.h"

#include "RequiredProgramMainCPPInclude.h"

#include "CEF3Utils.h"
#include "EpicWebHelperApp.h"

//#define DEBUG_USING_CONSOLE	0

IMPLEMENT_APPLICATION(EpicWebHelper, "EpicWebHelper")

#if WITH_CEF3
int32 RunCEFSubProcess(const CefMainArgs& MainArgs)
{
	bool bLoadedCEF = CEF3Utils::LoadCEF3Modules(false);
	check(bLoadedCEF);

	// Create an App object for handling various render process events, such as message passing
    CefRefPtr<CefApp> App(new FEpicWebHelperApp);

	// Execute the sub-process logic. This will block until the sub-process should exit.
	int32 Result = CefExecuteProcess(MainArgs, App, nullptr);
	CEF3Utils::UnloadCEF3Modules();
	return Result;
}
#endif
