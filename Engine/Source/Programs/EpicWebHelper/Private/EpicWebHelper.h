// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3
#include "EpicWebHelperLibCEFIncludes.h"

/**
 * The application's main function.
 *
 * @param MainArgs Main Arguments for the process (created differently on each platform).
 * @return Application's exit value.
 */
int32 RunCEFSubProcess(const CefMainArgs& MainArgs);
#endif
