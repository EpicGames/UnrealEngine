// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicWebHelperApp.h"
#include "EpicWebHelper.h"

#if WITH_CEF3

FEpicWebHelperApp::FEpicWebHelperApp()
{
}

void FEpicWebHelperApp::OnContextCreated( CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context )
{
	// Check if we have told the main browser process about this new browser
	if (PendingBrowserCreated[Browser->GetIdentifier()] == true)
	{
		int BrowserID = Browser->GetIdentifier();
		PendingBrowserCreated[BrowserID] = false;

		// tell the main process we have a new browser. This can happen if you navigate between top level sites in a browser (i.e store -> paypal)
		CefRefPtr<CefProcessMessage> Message = CefProcessMessage::Create("CEF::BROWSERCREATED");
		CefRefPtr<CefListValue> MessageArguments = Message->GetArgumentList();
		MessageArguments->SetInt(0, BrowserID);

		Frame->SendProcessMessage(PID_BROWSER, Message);
	}

	RemoteScripting.OnContextCreated(Browser, Frame, Context);
}

void FEpicWebHelperApp::OnContextReleased( CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context )
{
	RemoteScripting.OnContextReleased(Browser, Frame, Context);
}

void FEpicWebHelperApp::OnBrowserCreated(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefDictionaryValue> ExtraInfo)
{
	if (Browser.get() == nullptr )
	{
		return;
	}
	// record this create but don't send an IPC message here as Browser->GetMainFrame() can still be nullptr at this point
	PendingBrowserCreated.Add(Browser->GetIdentifier(), true);

}


bool FEpicWebHelperApp::OnProcessMessageReceived( CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message )
{
	bool Result = false;
	FString MessageName = WCHAR_TO_TCHAR(Message->GetName().ToWString().c_str());
	if (MessageName.StartsWith(TEXT("UE::")))
	{
		Result = RemoteScripting.OnProcessMessageReceived(Browser, SourceProcess, Message);
	}
	else if (MessageName.StartsWith(TEXT("CEF::STARTUP")))
	{
		CefRefPtr<CefListValue> MessageArguments = Message->GetArgumentList();
		for (size_t I = 0; I < MessageArguments->GetSize(); I++)
		{
			if (MessageArguments->GetType(I) == VTYPE_DICTIONARY)
			{
				CefRefPtr<CefDictionaryValue> Info = MessageArguments->GetDictionary(I);
				if (Info->GetType("browser") == VTYPE_INT)
				{
					int32 BrowserID = Info->GetInt("browser");

					CefRefPtr<CefDictionaryValue> Bindings = Info->GetDictionary("bindings");
					RemoteScripting.InitPermanentBindings(BrowserID, Bindings);

					// register the new permanent bindings we got. Note this may happen multiple times for a single browser
					if (Frame->GetV8Context().get() != nullptr && Bindings->GetSize() > 0)
					{
						CefRefPtr<CefV8Context> Context = Frame->GetV8Context();
						RemoteScripting.OnContextCreated(Browser, Frame, Context);
					}
				}
			}
		}
	}

	return Result;
}


#if !PLATFORM_LINUX
void FEpicWebHelperApp::OnFocusedNodeChanged(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefDOMNode> Node)
{
	if (Frame.get() == nullptr)
	{
		return;
	}
	
	CefRefPtr<CefProcessMessage> Message = CefProcessMessage::Create("UE::IME::FocusChanged");
	CefRefPtr<CefListValue> MessageArguments = Message->GetArgumentList();

	if (Node.get() == nullptr)
	{
		MessageArguments->SetString(0, "NONE");
	}
	else
	{
		static const TMap<uint32, CefString> DomNodeTypeStrings = []()
		{
			TMap<uint32, CefString> Result;
#define ADD_NODETYPE_STRING(NodeTypeCode) Result.Add(NodeTypeCode, #NodeTypeCode)
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_UNSUPPORTED);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_ELEMENT);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_ATTRIBUTE);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_TEXT);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_CDATA_SECTION);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_PROCESSING_INSTRUCTIONS);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_COMMENT);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_DOCUMENT);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_DOCUMENT_TYPE);
			ADD_NODETYPE_STRING(DOM_NODE_TYPE_DOCUMENT_FRAGMENT);
#undef ADD_NODETYPE_STRING
			return Result;
		}();

		MessageArguments->SetString(0, DomNodeTypeStrings[Node->GetType()]);
		MessageArguments->SetString(1, Node->GetName());
		MessageArguments->SetBool(2, Node->IsEditable());
		MessageArguments->SetString(3, Node->GetValue());
		MessageArguments->SetInt(4, Node->GetElementBounds().x);
		MessageArguments->SetInt(5, Node->GetElementBounds().y);
		MessageArguments->SetInt(6, Node->GetElementBounds().width);
		MessageArguments->SetInt(7, Node->GetElementBounds().height);
	}

	Frame->SendProcessMessage(PID_BROWSER, Message);
}
#endif

#endif
