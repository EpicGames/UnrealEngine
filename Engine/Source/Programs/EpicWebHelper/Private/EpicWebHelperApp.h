// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3

#include "EpicWebHelperLibCEFIncludes.h"


#include "EpicWebHelperRemoteScripting.h"

/**
 * Implements CEF App and other Process level interfaces
 */
class FEpicWebHelperApp
	: public CefApp
	, public CefRenderProcessHandler
{
public:
    
	/**
	 * Default Constructor
	 */
	FEpicWebHelperApp();

private:
	// CefApp methods:
	virtual CefRefPtr<CefRenderProcessHandler> GetRenderProcessHandler() override { return this; }

	// CefRenderProcessHandler methods:
	virtual void OnContextCreated( CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context ) override;

	virtual void OnContextReleased( CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context ) override;

	virtual bool OnProcessMessageReceived( CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message ) override;

	virtual void OnBrowserCreated(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefDictionaryValue> ExtraInfo) override;

#if !PLATFORM_LINUX
	virtual void OnFocusedNodeChanged(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefDOMNode> Node) override;
#endif

	// Handles remote scripting messages from the frontend process
	FEpicWebHelperRemoteScripting RemoteScripting;

	// Stores the browser ID and flag for any browser so we can track if we have told the main process about it
	TMap<int32, bool> PendingBrowserCreated;

	// Include the default reference counting implementation.
	IMPLEMENT_REFCOUNTING(FEpicWebHelperApp);
};

#endif // WITH_CEF3
