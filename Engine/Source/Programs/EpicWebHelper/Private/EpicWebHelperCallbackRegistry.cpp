// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicWebHelperCallbackRegistry.h"
#include "EpicWebHelper.h"

#if WITH_CEF3

FGuid FEpicWebHelperCallbackRegistry::FindOrAdd(CefRefPtr<CefV8Context> Context, CefRefPtr<CefV8Value> Object, CefRefPtr<CefV8Value> Function, CefRefPtr<CefV8Value> OnError, bool bOneShot)
{
	if (! bOneShot)
	{
		for(auto Iter = CreateIterator(); Iter; ++Iter)
		{
			auto Value = Iter.Value();
			if (! Value.bOneShot
				&& Context->IsSame(Value.Context)
				&& Function->IsSame(Value.Function)
				&& ( ((Object == nullptr) && (Value.Object == nullptr)) || Object->IsSame(Value.Object))
				&& ( ((OnError == nullptr) && (Value.OnError == nullptr)) || OnError->IsSame(Value.OnError)))
			{
				return Iter.Key();
			}
		}
	}

	// If not found or one-shot, add new entry to the map
	FGuid Guid = FGuid::NewGuid();
	Add(Guid, FEpicWebHelperCallbackRegistryEntry(Context, Object, Function, OnError, bOneShot));
	return Guid;
}

void FEpicWebHelperCallbackRegistry::RemoveByContext(CefRefPtr<CefV8Context> Context)
{
	for(auto Iter = CreateIterator(); Iter; ++Iter)
	{
		if (Context->IsSame(Iter.Value().Context))
		{
			Iter.RemoveCurrent();
		}
	}
}

#endif // WITH_CEF3
