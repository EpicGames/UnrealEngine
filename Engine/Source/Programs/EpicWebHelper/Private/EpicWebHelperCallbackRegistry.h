// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "EpicWebHelper.h"

#if WITH_CEF3

/** Represents information about a JS function that can be called from the browser process.
 */
struct FEpicWebHelperCallbackRegistryEntry
{
	FEpicWebHelperCallbackRegistryEntry(CefRefPtr<CefV8Context> InContext, CefRefPtr<CefV8Value> InObject, CefRefPtr<CefV8Value> InFunction, CefRefPtr<CefV8Value> InOnError, bool bInOneShot)
		: Context(InContext)
		, Object(InObject)
		, Function(InFunction)
		, OnError(InOnError)
		, bOneShot(bInOneShot)
	{}


	CefRefPtr<CefV8Context> Context;
	CefRefPtr<CefV8Value> Object;
	CefRefPtr<CefV8Value> Function;
	CefRefPtr<CefV8Value> OnError;
	bool bOneShot;
};

class FEpicWebHelperCallbackRegistry : public TMap<FGuid, FEpicWebHelperCallbackRegistryEntry>
{
public:
	/** 
	 * Looks for a matching entry in the registry or adds a new one, returning its Id.
	 * A matching entry must have the same Context, Object, Function and OnError object. One-shot callbacks will always result in a new entry , even if there is an exact match.
	 */
	FGuid FindOrAdd(CefRefPtr<CefV8Context> Context, CefRefPtr<CefV8Value> Object, CefRefPtr<CefV8Value> Function, CefRefPtr<CefV8Value> OnError=nullptr, bool bOneShot=false);

	/**
	 * Deletes all entries that have a matching context.
	 */
	void RemoveByContext(CefRefPtr<CefV8Context> Context);
};

#endif
