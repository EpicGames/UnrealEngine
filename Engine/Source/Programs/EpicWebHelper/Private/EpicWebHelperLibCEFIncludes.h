// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_CEF3

#ifndef OVERRIDE
#	define OVERRIDE override
#endif //OVERRIDE

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include "Windows/AllowWindowsPlatformAtomics.h"
#endif

THIRD_PARTY_INCLUDES_START

#	if PLATFORM_APPLE
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
#	endif //PLATFORM_APPLE

#	pragma push_macro("OVERRIDE")
#		undef OVERRIDE // cef headers provide their own OVERRIDE macro

#		include "include/cef_app.h"
#	pragma pop_macro("OVERRIDE")

#	if PLATFORM_APPLE
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#	endif //PLATFORM_APPLE

THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#	include "Windows/HideWindowsPlatformAtomics.h"
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

#endif //WITH_CEF3
