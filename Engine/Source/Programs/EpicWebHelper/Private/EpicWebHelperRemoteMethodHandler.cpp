// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicWebHelperRemoteMethodHandler.h"
#include "EpicWebHelper.h"
#include "EpicWebHelperRemoteScripting.h"

#if WITH_CEF3


bool FEpicWebHelperRemoteMethodHandler::Execute(const CefString&, CefRefPtr<CefV8Value> Object, const CefV8ValueList& Arguments, CefRefPtr<CefV8Value>& Retval, CefString& InException)
{
	return RemoteObject->ExecuteMethod(MethodName, Object, Arguments, Retval, InException);
}

bool FEpicWebHelperRemoteObject::ExecuteMethod(const CefString& MethodName, CefRefPtr<CefV8Value> Object, const CefV8ValueList& Arguments, CefRefPtr<CefV8Value>& Retval, CefString& InException)
{
	CefRefPtr<CefV8Context> Context = CefV8Context::GetCurrentContext();
	CefRefPtr<CefV8Exception> Exception;
	CefRefPtr<CefV8Value> PromiseObjects;

	// Run JS code that creates and unwraps a Promise object
	const bool bEvalSuccess = Context->Eval(
		"(function() " \
		"{ "
		"	var Accept, Reject, PromiseObject;" \
		"	PromiseObject = new Promise(function(InAccept, InReject) " \
		"	{"
		"		Accept = InAccept;" \
		"		Reject = InReject;" \
		"	});" \
		"	return [PromiseObject, Accept, Reject];" \
		"})()",
		CefString(),
		0,
		PromiseObjects,
		Exception);

	if (!bEvalSuccess)
	{
		InException = Exception->GetMessage();
		return false;
	}

	if (!(PromiseObjects.get() && PromiseObjects->IsArray() && PromiseObjects->GetArrayLength() == 3))
	{
		return false;
	}

	Retval = PromiseObjects->GetValue(0);
	CefRefPtr<CefV8Value> Accept = PromiseObjects->GetValue(1);
	CefRefPtr<CefV8Value> Reject = PromiseObjects->GetValue(2);

	check(Retval->IsObject());
	check(Accept->IsFunction());
	check(Reject->IsFunction());

	FGuid CallbackGuid = RemoteScripting->CallbackRegistry.FindOrAdd(Context, Retval, Accept, Reject, true);
	CefRefPtr<CefProcessMessage> Message = CefProcessMessage::Create("UE::ExecuteUObjectMethod");
	CefRefPtr<CefListValue> MessageArguments = Message->GetArgumentList();
	MessageArguments->SetString(0, CefString(TCHAR_TO_WCHAR(*ObjectId.ToString(EGuidFormats::Digits))));
	MessageArguments->SetString(1, MethodName);
	MessageArguments->SetString(2, CefString(TCHAR_TO_WCHAR(*CallbackGuid.ToString(EGuidFormats::Digits))));
	MessageArguments->SetList(3, RemoteScripting->V8ArrayToCef(Arguments));

	Browser->GetMainFrame()->SendProcessMessage(PID_BROWSER, Message);

	return true;
}

// Will notify the client that the uobject is no longer being referenced on the JS side
void FEpicWebHelperRemoteObject::ReleaseMethod()
{
	if (Browser.get() && Browser->GetMainFrame())
	{
		CefRefPtr<CefProcessMessage> Message = CefProcessMessage::Create("UE::ReleaseUObject");
		CefRefPtr<CefListValue> MessageArguments = Message->GetArgumentList();
		MessageArguments->SetString(0, CefString(TCHAR_TO_WCHAR(*ObjectId.ToString(EGuidFormats::Digits))));
		Browser->GetMainFrame()->SendProcessMessage(PID_BROWSER, Message);
		//fprintf(stderr, "Releasing UOBject\n"); // debug spew useful for IPC
	}
}

#endif
