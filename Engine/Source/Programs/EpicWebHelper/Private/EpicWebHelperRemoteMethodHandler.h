// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#if WITH_CEF3
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif

#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_v8.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class FEpicWebHelperRemoteScripting;

class FEpicWebHelperRemoteObject
	: public CefBaseRefCounted
{
public:
	FEpicWebHelperRemoteObject(FEpicWebHelperRemoteScripting* InRemoteScripting, CefRefPtr<CefBrowser> InBrowser, CefRefPtr<CefV8Context> InContext, const FGuid& InObjectId)
		: RemoteScripting(InRemoteScripting)
		, Browser(InBrowser)
		, CreationContext(InContext)
		, ObjectId(InObjectId) {}

	bool ExecuteMethod(const CefString& MethodName,
		CefRefPtr<CefV8Value> Object, const CefV8ValueList& Arguments,
		CefRefPtr<CefV8Value>& Retval, CefString& Exception);

	void ReleaseMethod();

private:
	FEpicWebHelperRemoteScripting* RemoteScripting;
	CefRefPtr<CefBrowser> Browser;
	CefRefPtr<CefV8Context> CreationContext;
	FGuid ObjectId;

	friend class FEpicWebHelperRemoteScripting;
    // Include the default reference counting implementation.
    IMPLEMENT_REFCOUNTING(FEpicWebHelperRemoteObject);
};

class FEpicWebHelperRemoteMethodHandler
	: public CefV8Handler
{

public:
	FEpicWebHelperRemoteMethodHandler(CefRefPtr<FEpicWebHelperRemoteObject> InRemoteObject, CefString& InMethodName)
		: RemoteObject(InRemoteObject)
		, MethodName(InMethodName)
	{}

	virtual bool Execute(const CefString& Name,
		CefRefPtr<CefV8Value> Object, const CefV8ValueList& Arguments,
		CefRefPtr<CefV8Value>& Retval, CefString& Exception) override;

private:
	CefRefPtr<FEpicWebHelperRemoteObject> RemoteObject;
	CefString MethodName;

    // Include the default reference counting implementation.
    IMPLEMENT_REFCOUNTING(FEpicWebHelperRemoteMethodHandler);
};

#endif
