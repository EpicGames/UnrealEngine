// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicWebHelperRemoteScripting.h"
#include "EpicWebHelper.h"
#include "EpicWebHelperRemoteMethodHandler.h"
#if WITH_CEF3

CefRefPtr<CefV8Value> FEpicWebHelperRemoteScripting::CefToV8(CefRefPtr<CefDictionaryValue> Dictionary)
{
	// Custom types are encoded inside dictionary values with a $type and a $value property
	if ( Dictionary->GetType("$type") == VTYPE_STRING)
	{
		FString Type = WCHAR_TO_TCHAR(Dictionary->GetString("$type").ToWString().c_str());
		if ( Type == "struct" &&  Dictionary->GetType("$value") == VTYPE_DICTIONARY)
		{
			return CefToPlainV8Object(Dictionary->GetDictionary("$value"));
		}
		if ( Type == "uobject" && Dictionary->GetType("$id") == VTYPE_STRING && Dictionary->GetType("$methods") == VTYPE_LIST)
		{
			FGuid Guid;
			if (FGuid::Parse(WCHAR_TO_TCHAR(Dictionary->GetString("$id").ToWString().c_str()), Guid))
			{
				CefRefPtr<CefListValue> Methods = Dictionary->GetList("$methods");
				return CreateUObjectProxy(Guid, Methods);
			}
		}
	}
	return CefToPlainV8Object(Dictionary);

}

CefRefPtr<CefV8Value> FEpicWebHelperRemoteScripting::CreateUObjectProxy(FGuid ObjectId, CefRefPtr<CefListValue> Methods)
{
	CefRefPtr<CefV8Context> Context = CefV8Context::GetCurrentContext();
	CefRefPtr<CefBrowser> Browser = Context->GetBrowser();
	CefRefPtr<CefV8Value> Result = CefV8Value::CreateObject(nullptr, nullptr);
	CefRefPtr<FEpicWebHelperRemoteObject> Remote = new FEpicWebHelperRemoteObject(this, Browser, Context, ObjectId);
	for (size_t I=0; I < Methods->GetSize(); ++I)
	{
		CefString MethodName = Methods->GetString(I);
		CefRefPtr<CefV8Value> FunctionProxy = CefV8Value::CreateFunction(MethodName, new FEpicWebHelperRemoteMethodHandler(Remote, MethodName));
		Result->SetValue(MethodName, FunctionProxy, static_cast<CefV8Value::PropertyAttribute>(V8_PROPERTY_ATTRIBUTE_DONTDELETE | V8_PROPERTY_ATTRIBUTE_READONLY ));
	}
	RemoteObjects.Add(ObjectId,Remote);
	Result->SetValue("$id", CefV8Value::CreateString(TCHAR_TO_WCHAR(*ObjectId.ToString(EGuidFormats::Digits))),  static_cast<CefV8Value::PropertyAttribute>(V8_PROPERTY_ATTRIBUTE_DONTDELETE | V8_PROPERTY_ATTRIBUTE_READONLY | V8_PROPERTY_ATTRIBUTE_DONTENUM));
	return Result;
}


CefRefPtr<CefV8Value> FEpicWebHelperRemoteScripting::CefToPlainV8Object(CefRefPtr<CefDictionaryValue> Dictionary)
{
	CefRefPtr<CefV8Value> Result = CefV8Value::CreateObject(nullptr, nullptr);
	CefDictionaryValue::KeyList Keys;
	Dictionary->GetKeys(Keys);
	for (CefString Key : Keys)
	{
		Result->SetValue(Key, CefToV8(Dictionary, Key), V8_PROPERTY_ATTRIBUTE_NONE);
	}
	return Result;
}

CefRefPtr<CefV8Value> FEpicWebHelperRemoteScripting::CefToV8(CefRefPtr<CefListValue> List)
{
	CefRefPtr<CefV8Value> Result = CefV8Value::CreateArray(List->GetSize());
	for (size_t i = 0; i < List->GetSize(); ++i)
	{
		Result->SetValue(i, CefToV8(List, i));
	}
	return Result;
}

void FEpicWebHelperRemoteScripting::CefToV8Arglist(CefRefPtr<CefListValue> List, CefV8ValueList& Values)
{
	for (size_t i = 0; i < List->GetSize(); ++i)
	{
		Values.push_back(CefToV8(List, i));
	}
}


CefRefPtr<CefListValue> FEpicWebHelperRemoteScripting::V8ArrayToCef(const CefV8ValueList& Values)
{
	CefRefPtr<CefListValue> Result = CefListValue::Create();
	for (size_t I = 0; I < Values.size(); ++I)
	{
		V8ToCef(Result, nullptr, I, Values[I]);
	}
	return Result;
}

CefRefPtr<CefListValue> FEpicWebHelperRemoteScripting::V8ArrayToCef(CefRefPtr<CefV8Value> Array)
{
	CefRefPtr<CefListValue> Result = CefListValue::Create();
	if (Array->IsArray())
	{
		for (int I = 0; I < Array->GetArrayLength(); ++I)
		{
			V8ToCef(Result, Array, I, Array->GetValue(I));
		}
	}
	return Result;
}

CefRefPtr<CefDictionaryValue> FEpicWebHelperRemoteScripting::V8ObjectToCef(CefRefPtr<CefV8Value> Object)
{
	CefRefPtr<CefDictionaryValue> Result = CefDictionaryValue::Create();
	if (Object->IsObject())
	{
		std::vector<CefString> Keys;
		Object->GetKeys(Keys);
		for (CefString Key : Keys)
		{
			V8ToCef(Result, Object, Key, Object->GetValue(Key));
		}
	}
	return Result;
}

CefRefPtr<CefDictionaryValue> FEpicWebHelperRemoteScripting::V8FunctionToCef(CefRefPtr<CefV8Value> Object, CefRefPtr<CefV8Value> Function)
{
	CefRefPtr<CefDictionaryValue> Result = CefDictionaryValue::Create();
	FGuid Guid = CallbackRegistry.FindOrAdd(CefV8Context::GetCurrentContext(), Object, Function);
	Result->SetString("$type", "callback");
	Result->SetString("$id", TCHAR_TO_WCHAR(*Guid.ToString(EGuidFormats::Digits)));
	Result->SetString("$name", Function->GetFunctionName());
	return Result;
}

bool FEpicWebHelperRemoteScripting::HandleExecuteJSFunctionMessage(CefRefPtr<CefListValue> MessageArguments)
{
	FGuid Guid;

	// Message arguments are CallbackGuid, FunctionArguments, bIsError
	if (MessageArguments->GetSize() != 3
		|| MessageArguments->GetType(0) != VTYPE_STRING
		|| MessageArguments->GetType(1) != VTYPE_LIST
		|| MessageArguments->GetType(2) != VTYPE_BOOL)
	{
		// Wrong message argument types or count
		return false;
	}

	if (!FGuid::Parse(FString(WCHAR_TO_TCHAR(MessageArguments->GetString(0).ToWString().c_str())), Guid))
	{
		// Invalid GUID
		return false;
	}
	if (!CallbackRegistry.Contains(Guid))
	{
		// Unknown callback id
		return false;
	}

	auto Callback = CallbackRegistry[Guid];
	{
		ScopedV8Context ContextScope(Callback.Context);

		bool bIsErrorCallback = MessageArguments->GetBool(2);
		CefRefPtr<CefV8Value> Function = bIsErrorCallback?Callback.OnError:Callback.Function;

		if (!Function.get())
		{
			// Either invalid entry or no error handler
			return false;
		}

		CefV8ValueList FunctionArguments;
		CefToV8Arglist(MessageArguments->GetList(1), FunctionArguments);
		CefRefPtr<CefV8Value> Retval = Function->ExecuteFunction(Callback.Object, FunctionArguments);
		if (!Retval.get())
		{
			// Function call resulted in an error
			return false;
		}

		// Remove callback if it's a one-shot callback and successful.
		if (Callback.bOneShot)
		{
			CallbackRegistry.Remove(Guid);
		}
		return true;
	}
}

bool FEpicWebHelperRemoteScripting::HandleSetValueMessage(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefListValue> MessageArguments)
{
	CefRefPtr<CefFrame> MainFrame = Browser->GetMainFrame();
	CefRefPtr<CefV8Context> Context = MainFrame->GetV8Context();
	ScopedV8Context ContextScope(Context);
	CefRefPtr<CefV8Value> RootObject = Context->GetGlobal()->GetValue("ue");
	if (!RootObject.get())
	{
		// The root object should always be created on context creation.
		return false;
	}

	for (size_t I = 0; I < MessageArguments->GetSize(); I++)
	{
		if (MessageArguments->GetType(I) != VTYPE_DICTIONARY)
		{
			return false;
		}
		CefRefPtr<CefDictionaryValue> Argument = MessageArguments->GetDictionary(I);

		if (Argument->GetType("name") != VTYPE_STRING
			|| Argument->GetType("value") != VTYPE_DICTIONARY
			|| Argument->GetType("permanent") != VTYPE_BOOL)
		{
			// Wrong message argument types or count
			return false;
		}

		CefString Name = Argument->GetString("name");
		CefRefPtr<CefDictionaryValue> CefValue = Argument->GetDictionary("value");
		bool bPermanent = Argument->GetBool("permanent");

		if (bPermanent)
		{
			int32 BrowserID = Browser->GetIdentifier();
			CefRefPtr<CefDictionaryValue> Bindings;
			if (PermanentBindings.Contains(BrowserID))
			{
				Bindings = PermanentBindings[BrowserID];
			}
			else
			{
				Bindings = CefDictionaryValue::Create();
				PermanentBindings.Add(BrowserID, Bindings);
			}

			Bindings->SetDictionary(Name, CefValue);
		}
		CefRefPtr<CefV8Value> Value = CefToV8(CefValue);
		RootObject->SetValue(Name, Value, V8_PROPERTY_ATTRIBUTE_NONE);

	}
	return true;
}

bool FEpicWebHelperRemoteScripting::HandleDeleteValueMessage(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefListValue> MessageArguments)
{
	CefRefPtr<CefFrame> MainFrame = Browser->GetMainFrame();
	CefRefPtr<CefV8Context> Context = MainFrame->GetV8Context();
	ScopedV8Context ContextScope(Context);
	CefRefPtr<CefV8Value> RootObject = Context->GetGlobal()->GetValue("ue");
	if (!RootObject.get())
	{
		// The root object should always be created on context creation.
		return false;
	}

	for (size_t I = 0; I < MessageArguments->GetSize(); I++)
	{
		if (MessageArguments->GetType(I) != VTYPE_DICTIONARY)
		{
			return false;
		}
		CefRefPtr<CefDictionaryValue> Argument = MessageArguments->GetDictionary(I);

		if (Argument->GetType("name") != VTYPE_STRING
			|| Argument->GetType("id") != VTYPE_STRING
			|| Argument->GetType("permanent") != VTYPE_BOOL)
		{
			// Wrong message argument types or count
			return false;
		}

		CefString Name = Argument->GetString("name");
		CefString Id = Argument->GetString("id");
		bool bPermanent = Argument->GetBool("permanent");

		FGuid Guid;
		if (!FGuid::Parse(WCHAR_TO_TCHAR(Id.ToWString().c_str()), Guid))
		{
			return false;
		}

		if (bPermanent)
		{
			int32 BrowserID = Browser->GetIdentifier();
			CefRefPtr<CefDictionaryValue> Bindings;
			if (PermanentBindings.Contains(BrowserID))
			{
				Bindings = PermanentBindings[BrowserID];
				if (!Bindings->HasKey(Name))
				{
					return false;
				}
				if (Guid.IsValid())
				{
					CefRefPtr<CefDictionaryValue> CefValue = Bindings->GetDictionary(Name);
					if (CefValue.get() && CefValue->GetString("$id") != Id)
					{
						return false;
					}
				}
				Bindings->Remove(Name);
			}
		}

		if (!RootObject->HasValue(Name))
		{
			return false;
		}
		if (Guid.IsValid())
		{
			CefRefPtr<CefV8Value> Value = RootObject->GetValue(Name);
			if (!Value->HasValue("$id") || !Value->GetValue("$id")->IsString() || Value->GetValue("$id")->GetStringValue() != Id)
			{
				return false;
			}
		}
		RootObject->DeleteValue(Name);
	}
	return true;

}


bool FEpicWebHelperRemoteScripting::OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser,
    CefProcessId SourceProcess,
    CefRefPtr<CefProcessMessage> Message)
{
	bool Result = false;
	FString MessageName = WCHAR_TO_TCHAR(Message->GetName().ToWString().c_str());
	if (MessageName == TEXT("UE::ExecuteJSFunction"))
	{
		Result = HandleExecuteJSFunctionMessage(Message->GetArgumentList());
	}
	else if (MessageName == TEXT("UE::SetValue"))
	{
		Result = HandleSetValueMessage(Browser, Message->GetArgumentList());
	}
	else if (MessageName == TEXT("UE::DeleteValue"))
	{
		Result = HandleDeleteValueMessage(Browser, Message->GetArgumentList());
	}

	return Result;
}

void FEpicWebHelperRemoteScripting::OnContextCreated(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context)
{
	ScopedV8Context ContextScope(Context);
	CefRefPtr<CefV8Value> Global = Context->GetGlobal();
	if ( !Global->HasValue("ue") )
	{
		Global->SetValue("ue", CefV8Value::CreateObject(nullptr, nullptr), V8_PROPERTY_ATTRIBUTE_DONTDELETE);
	}
	CefRefPtr<CefV8Value> RootObject = Context->GetGlobal()->GetValue("ue");

	int32 BrowserID = Browser->GetIdentifier();

	if (PermanentBindings.Contains(BrowserID))
	{
		CefRefPtr<CefDictionaryValue> Bindings = PermanentBindings[BrowserID];
		CefDictionaryValue::KeyList Keys;
		Bindings->GetKeys(Keys);
		for (CefString Key : Keys)
		{
			CefRefPtr<CefV8Value> Value = CefToV8(Bindings->GetDictionary(Key));
			RootObject->SetValue(Key, Value, V8_PROPERTY_ATTRIBUTE_NONE);
		}
	}
}

void FEpicWebHelperRemoteScripting::OnContextReleased(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context)
{
	for(const TPair<FGuid,CefRefPtr<FEpicWebHelperRemoteObject>> &RemoteObject : RemoteObjects)
	{
		if (RemoteObject.Value->Browser->IsSame(Browser)
			&&  RemoteObject.Value->CreationContext->IsSame(Context))
		{
			RemoteObject.Value->ReleaseMethod();
			RemoteObjects.Remove(RemoteObject.Key);
			break;
		}
	}
	// Invalidate JS functions that were created in the context being released.
	CallbackRegistry.RemoveByContext(Context);
}

void FEpicWebHelperRemoteScripting::InitPermanentBindings(int32 BrowserID, CefRefPtr<CefDictionaryValue> Values)
{
	// The CefDictionary in PermanentBindings needs to be writable, so we'll copy the Values argument before saving it.
	PermanentBindings.Add(BrowserID, Values->Copy(true));
}


#endif
