// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif

#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_app.h"
#include "include/cef_v8.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "EpicWebHelperCallbackRegistry.h"
#include "EpicWebHelperRemoteMethodHandler.h"


class FEpicWebHelperRemoteScripting
{

public:

	FEpicWebHelperRemoteScripting()
	{}

	CefRefPtr<CefV8Value> CefToV8(CefRefPtr<CefDictionaryValue> Dictionary);
	CefRefPtr<CefV8Value> CefToV8(CefRefPtr<CefListValue> List);
	void CefToV8Arglist(CefRefPtr<CefListValue> List, CefV8ValueList& Values);
	// Works for CefListValue and CefDictionaryValues
	template<typename ContainerType, typename KeyType>
	CefRefPtr<CefV8Value> CefToV8(CefRefPtr<ContainerType> Container, KeyType Key)
	{
		switch(Container->GetType(Key))
		{
			case VTYPE_NULL:
				return CefV8Value::CreateNull();
			case VTYPE_BOOL:
				return CefV8Value::CreateBool(Container->GetBool(Key));
			case VTYPE_INT:
				return CefV8Value::CreateInt(Container->GetInt(Key));
			case VTYPE_DOUBLE:
				return CefV8Value::CreateDouble(Container->GetDouble(Key));
			case VTYPE_STRING:
				return CefV8Value::CreateString(Container->GetString(Key));
			case VTYPE_DICTIONARY:
				return CefToV8(Container->GetDictionary(Key));
			case VTYPE_LIST:
				return CefToV8(Container->GetList(Key));
			default:
				return nullptr;
		}

	}

	template<typename ContainerType, typename KeyType>
	bool V8ToCef(CefRefPtr<ContainerType> Container, CefRefPtr<CefV8Value> Parent, KeyType Key, CefRefPtr<CefV8Value> Value)
	{
		if(Value->IsNull())
		{
			return Container->SetNull(Key);
		}
		else if(Value->IsUndefined())
		{
			return Container->SetNull(Key); // TODO distinguish between NULL and undef?
		}
		else if(Value->IsBool())
		{
			return Container->SetBool(Key, Value->GetBoolValue());
		}
		else if(Value->IsDouble())
		{
			return Container->SetDouble(Key, Value->GetDoubleValue());
		}
		else if(Value->IsInt())
		{
			return Container->SetInt(Key, Value->GetIntValue());
		}
		else if(Value->IsString())
		{
			return Container->SetString(Key, Value->GetStringValue());
		}
		else if(Value->IsUInt())
		{
			return Container->SetDouble(Key, Value->GetUIntValue());
		}
		else if(Value->IsDate())
		{
			return Container->SetNull(Key); // TODO
		}
		else if(Value->IsFunction())
		{
			return Container->SetDictionary(Key, V8FunctionToCef(Parent, Value));
		}
		else if(Value->IsArray())
		{
			return Container->SetList(Key, V8ArrayToCef(Value));
		}
		else if(Value->IsObject())
		{
			return Container->SetDictionary(Key, V8ObjectToCef(Value));
		}
		else
		{
			return Container->SetNull(Key);
		}
	}
	CefRefPtr<CefListValue> V8ArrayToCef(const CefV8ValueList& Values);
	CefRefPtr<CefListValue> V8ArrayToCef(CefRefPtr<CefV8Value> Value);
	CefRefPtr<CefDictionaryValue> V8ObjectToCef(CefRefPtr<CefV8Value> Value);
	CefRefPtr<CefDictionaryValue> V8FunctionToCef(CefRefPtr<CefV8Value> Object, CefRefPtr<CefV8Value> Function);


    bool OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message);
	void OnContextCreated(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context);
	void OnContextReleased(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefV8Context> Context);
	void InitPermanentBindings(int32 BrowserID, CefRefPtr<CefDictionaryValue> Values);

private:
	// Scripting integration message handlers:
	bool HandleExecuteJSFunctionMessage(CefRefPtr<CefListValue> MessageArguments);
	bool HandleSetValueMessage(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefListValue> MessageArguments);
	bool HandleDeleteValueMessage(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefListValue> MessageArguments);

	/** Deletes information about a UObject on the client side.
	 */
	bool HandleUnRegisterUObjectMessage(CefRefPtr<CefListValue> MessageArguments);

	CefRefPtr<CefV8Value> CefToPlainV8Object(CefRefPtr<CefDictionaryValue> Dictionary);
	CefRefPtr<CefV8Value> CreateUObjectProxy(FGuid ObjectId, CefRefPtr<CefListValue> Methods);

	// Stores information about functions that can be called from the remote process.
	FEpicWebHelperCallbackRegistry CallbackRegistry;

	TMap<int32,CefRefPtr<CefDictionaryValue>> PermanentBindings;
	TMap<FGuid,CefRefPtr<FEpicWebHelperRemoteObject>> RemoteObjects;

	class ScopedV8Context
	{
	public:
		ScopedV8Context(CefRefPtr<CefV8Context> InContext)
			: Context(InContext)
		{
			Context->Enter();
		}
		~ScopedV8Context()
		{
			Context->Exit();
		}
	private:
		CefRefPtr<CefV8Context> Context;
	};

	friend class FEpicWebHelperRemoteObject;
};

#endif
