// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicWebHelper.h"

/**
 * WinMain, called when the application is started
 */
int main( int argc, char** argv )
{
	FPlatformMisc::SetGracefulTerminationHandler();

#if WITH_CEF3
	// Structure for passing command-line arguments.
	// The definition of this structure is platform-specific.
	CefMainArgs MainArgs(argc, argv);

	return RunCEFSubProcess(MainArgs);
#else
	return 0;
#endif
}
