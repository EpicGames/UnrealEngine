// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicWebHelper.h"

/**
 * WinMain, called when the application is started
 */
int WINAPI WinMain( _In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR, _In_ int nCmdShow )
{
#if WITH_CEF3
	// Structure for passing command-line arguments.
	// The definition of this structure is platform-specific.
	CefMainArgs MainArgs(hInInstance);

	return RunCEFSubProcess(MainArgs);
#else
	return 0;
#endif
}
